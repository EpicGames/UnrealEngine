// Copyright Epic Games, Inc. All Rights Reserved.

#include "Model.h"

#include "Trace/Analyzer.h"
#include "TraceServices/Model/Definitions.h"
#include "TraceServices/Model/Strings.h"
#include "TraceServices/Model/MetadataProvider.h"

namespace UE::HttpInsights
{

using namespace TraceServices;

////////////////////////////////////////////////////////////////////////////////////////////////////
enum class EHttpTraceEventId : uint16
{
	DispatcherCreated,
	CategoryCreated,
	RequestStarted,
	ChunkRangeAdded,
	RequestCompleted,
	PackageIdMapped,
};

////////////////////////////////////////////////////////////////////////////////////////////////////
class FHttpAnalyzer
	: public UE::Trace::IAnalyzer 
{
public:
	FHttpAnalyzer(IAnalysisSession& InSession, IHttpLogModel& InLogModel)
		: Session(InSession)
		, LogModel(InLogModel)
	{ }

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	IAnalysisSession&	Session;
	IHttpLogModel&		LogModel;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
void FHttpAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	Trace::IAnalyzer::FInterfaceBuilder& Builder = Context.InterfaceBuilder;
	Builder.RouteEvent(uint16(EHttpTraceEventId::DispatcherCreated),	"Http",		"DispatcherCreated");
	Builder.RouteEvent(uint16(EHttpTraceEventId::CategoryCreated),		"Http", 	"CategoryCreated");
	Builder.RouteEvent(uint16(EHttpTraceEventId::RequestStarted),		"Http", 	"RequestStarted");
	Builder.RouteEvent(uint16(EHttpTraceEventId::ChunkRangeAdded),		"Http", 	"ChunkRangeAdded");
	Builder.RouteEvent(uint16(EHttpTraceEventId::RequestCompleted),		"Http", 	"RequestCompleted");
	Builder.RouteEvent(uint16(EHttpTraceEventId::PackageIdMapped),		"Package",	"PackageMapping");
}

bool FHttpAnalyzer::OnEvent(uint16 EventIdValue, EStyle Style, const FOnEventContext& Context)
{
	const auto& EventData			= Context.EventData;
	const EHttpTraceEventId EventId = static_cast<EHttpTraceEventId>(EventIdValue);

	switch(EventId)
	{
	case EHttpTraceEventId::DispatcherCreated:
	{
		FAnsiStringView Name;
		EventData.GetString("Name", Name);
		LogModel.ProcesEvent(FHttpDispatcherCreated
		{
			.Dispatcher = EventData.GetValue<uint64>("Dispatcher"),
			.Name		= FAnsiString(Name)
		});
	}
	break;

	case EHttpTraceEventId::CategoryCreated:
	{
		FAnsiStringView Name;
		EventData.GetString("Name", Name);
		LogModel.ProcesEvent(FHttpCategoryCreated
		{
			.Category	= EventData.GetValue<uint8>("Category"),
			.Name		= FAnsiString(Name)
		});
	}
	break;

	case EHttpTraceEventId::RequestStarted:
	{
		const uint64 Cycles = EventData.GetValue<uint64>("StartTime");
		FAnsiStringView Url;
		EventData.GetString("Url", Url);

		LogModel.ProcesEvent(FHttpRequestStarted
		{
			.Dispatcher = EventData.GetValue<uint64>("Dispatcher"),
			.Request	= EventData.GetValue<uint64>("Request"),
			.StartTime	= Context.EventTime.AsSeconds(Cycles),
			.Priority	= EventData.GetValue<int32>("Priority"),
			.Category	= EventData.GetValue<uint32>("Category"),
			.Url		= FAnsiString(Url)
		});
	}
	break;

	case EHttpTraceEventId::ChunkRangeAdded:
	{
		FHttpChunkRangeAdded ChunkRangeAdded = FHttpChunkRangeAdded
		{
			.Request	= EventData.GetValue<uint64>("Request"),
			.Start		= EventData.GetValue<uint32>("Start"),
			.End		= EventData.GetValue<uint32>("End")
		};

		const uint32 ChunkIdSize = uint32(sizeof(FIoChunkId));
		const TArrayReader<uint8>& Reader = EventData.GetArray<uint8>("ChunkId");
		if (Reader.Num() == ChunkIdSize) 
		{
			FMemory::Memcpy(&ChunkRangeAdded.ChunkId, Reader.GetData(), ChunkIdSize);
		}
		LogModel.ProcesEvent(MoveTemp(ChunkRangeAdded));
	}
	break;

	case EHttpTraceEventId::RequestCompleted:
	{
		const uint64 Cycles = EventData.GetValue<uint64>("CompletionTime");
		FAnsiStringView Host;
		EventData.GetString("Host", Host);

		LogModel.ProcesEvent(FHttpRequestCompleted
		{
			.Request		= EventData.GetValue<uint64>("Request"),
			.CompletionTime	= Context.EventTime.AsSeconds(Cycles),
			.StatusCode		= EventData.GetValue<uint32>("StatusCode"),
			.ContentLength	= EventData.GetValue<uint32>("ContentLength"),
			.Host			= FAnsiString(Host)
		});
	}
	break;

	case EHttpTraceEventId::PackageIdMapped:
	{
		if (const IDefinitionProvider* DefinitionProvider = TraceServices::ReadDefinitionProvider(Session))
		{
			FPackageId PackageId		= FPackageId::FromValue(EventData.GetValue<uint64>("Id"));
			const auto PackageNameRef	= EventData.GetReferenceValue<uint32>("Package");
			const auto PackageName		= DefinitionProvider->Get<FStringDefinition>(PackageNameRef);

			if (PackageName && *PackageName->Display != '\0')
			{
				LogModel.ProcesEvent(FPackageIdMapped
				{
					.PackageId		= PackageId,
					.PackageName	= PackageName->Display
				});
			}
		}
	}
	break;

	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
TUniquePtr<UE::Trace::IAnalyzer> MakeHttpAnalyzer(IAnalysisSession& Session, IHttpLogModel& LogModel)
{
	return MakeUnique<FHttpAnalyzer>(Session, LogModel);
}

} // namespace UE:Http::Insights
