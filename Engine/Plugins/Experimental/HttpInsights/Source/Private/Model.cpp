// Copyright Epic Games, Inc. All Rights Reserved.

#include "Model.h"

#include "Async/Mutex.h"
#include "Async/SharedLock.h"
#include "Async/SharedMutex.h"
#include "Async/UniqueLock.h"
#include "Containers/PagedArray.h"
#include "HttpInsights/ModuleInterface.h"

namespace UE::HttpInsights
{

using namespace TraceServices;

////////////////////////////////////////////////////////////////////////////////
static FPackageId GetPackageId(const FIoChunkId& ChunkId)
{
	FPackageId Out;
	FMemory::Memcpy(&Out, &ChunkId, sizeof(FPackageId));
	return Out;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
class FPackageNameCache
{
	using FLookup = TMap<FPackageId, const TCHAR*>;

public:
	void Put(const FPackageId& PackageId, const TCHAR* PackageName)
	{
		TUniqueLock Lock(Mutex);
		Lookup.Add(PackageId, PackageName);
	}

	const TCHAR* Get(const FIoChunkId& ChunkId) const
	{
		const FPackageId PackageId = GetPackageId(ChunkId);
		TUniqueLock Lock(Mutex);
		return Lookup.FindRef(PackageId);
	}

private:
	FLookup			Lookup;
	mutable FMutex	Mutex;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
class FHttpLogModel
	: public IHttpLogModel
{
	struct FCategory
	{
		FString Name;
		uint8	Value;
	};

	using FPendingReqeusts	= TMap<uint64, FHttpRequestStarted>;
	using FCategories		= TPagedArray<FCategory>;
	using FPackageMappings	= TMap<FPackageId, const TCHAR*>;

public:
							FHttpLogModel(IAnalysisSession& InSession);
	// Writer
	virtual void			ProcesEvent(FHttpDispatcherCreated&& Event) override;
	virtual void			ProcesEvent(FHttpCategoryCreated&&) override;
	virtual void			ProcesEvent(FHttpRequestStarted&& Event) override;
	virtual void			ProcesEvent(FHttpChunkRangeAdded&& Event) override;
	virtual void			ProcesEvent(FHttpRequestCompleted&& Event) override;
	virtual void			ProcesEvent(FPackageIdMapped&&) override;
	// Reader
	virtual int32			IterateLog(int32 StartIndex, IHttpLogModel::FReadLogFunc&&) const override;
	virtual const TCHAR*	GetPackageName(const FIoChunkId& ChunkId) const override;

private:
	const FHttpDispatcher&	FindDispatcher(uint64 Handle);
	const FCategory&		FindCategory(uint8 CategoryValue) const;

	IAnalysisSession&				Session;
	TPagedArray<FHttpDispatcher>	Dispatchers;
	TPagedArray<FHttpRequest>		Requests;
	TPagedArray<FHttpChunkRange>	ChunkRanges;
	FCategories						Categories;
	FPackageNameCache				PackageNameCache;
	FPendingReqeusts				PendingRequests;
	mutable FSharedMutex			Mutex;
	uint32							SeqNo = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
FHttpLogModel::FHttpLogModel(IAnalysisSession& InSession)
	: Session(InSession)
{
	Dispatchers.Add(FHttpDispatcher
	{
		.Handle = MAX_uint64,
		.Name	= TEXT("Unknown")
	});
	
	Categories.Add(FCategory
	{
		.Name		= TEXT("Unknown"),
		.Value		= MAX_uint8
	});
}

void FHttpLogModel::ProcesEvent(FHttpDispatcherCreated&& Event)
{
	Dispatchers.Add(FHttpDispatcher
	{
		.Handle = Event.Dispatcher,
		.Name	= FString(Event.Name)
	});
}

void FHttpLogModel::ProcesEvent(FHttpCategoryCreated&& Event)
{
	Categories.Add(FCategory
	{
		.Name		= FString(Event.Name),
		.Value		= Event.Category
	});
}

void FHttpLogModel::ProcesEvent(FHttpRequestStarted&& Event)
{
	if (FHttpRequestStarted* Pending = PendingRequests.Find(Event.Request); Pending == nullptr)
	{
		PendingRequests.Add(Event.Request, MoveTemp(Event));
	}
	else
	{
		UE_LOGF(LogHttpInsights, Warning, "Request already pending");
	}
}

void FHttpLogModel::ProcesEvent(FHttpChunkRangeAdded&& Event)
{
	if (FHttpRequestStarted* Pending = PendingRequests.Find(Event.Request); Pending != nullptr)
	{
		FHttpChunkRange* Next = Pending->ChunkRanges;
		Pending->ChunkRanges = &ChunkRanges.Add_GetRef(FHttpChunkRange
		{
			.Next			= Next,
			.ChunkId		= Event.ChunkId,
			.Start			= Event.Start,
			.End			= Event.End
		});
	}
	else
	{
		UE_LOGF(LogHttpInsights, Warning, "Failed to append chunk range info to pending request");
	}
}

void FHttpLogModel::ProcesEvent(FHttpRequestCompleted&& Event)
{
	if (FHttpRequestStarted* Pending = PendingRequests.Find(Event.Request); Pending != nullptr)
	{
		const uint8 CategoryValue = static_cast<uint8>(Pending->Category);
		const FCategory& Category = FindCategory(CategoryValue);
		{
			TUniqueLock Lock(Mutex);
			Requests.Add(FHttpRequest
			{
				.Dispatcher		= FindDispatcher(Pending->Dispatcher),
				.StartTime		= Pending->StartTime,
				.CompletionTime	= Event.CompletionTime,
				.StatusCode		= Event.StatusCode,
				.ContentLength	= Event.ContentLength,
				.SeqNo			= ++SeqNo,
				.ChunkRanges	= MoveTemp(Pending->ChunkRanges),
				.Host			= FString(Event.Host),
				.Url			= FString(Pending->Url),
				.CategoryName	= &Category.Name,
				.Category		= CategoryValue
			});
		}
		PendingRequests.Remove(Event.Request);
	}
	else
	{
		UE_LOGF(LogHttpInsights, Warning, "Unknown completed request");
	}
}

void FHttpLogModel::ProcesEvent(FPackageIdMapped&& Event)
{
	PackageNameCache.Put(Event.PackageId, Event.PackageName);
}

int32 FHttpLogModel::IterateLog(int32 StartIndex, IHttpLogModel::FReadLogFunc&& Read) const
{
	int32 Index = StartIndex < 0 ? 0 : StartIndex;

	TSharedLock Lock(Mutex);

	if (Requests.IsEmpty())
	{
		return INDEX_NONE;
	}

	for (int32 Num = Requests.Num(); Index < Num; ++Index)
	{
		Read(Requests[Index]);
	}

	return Index;
}

const FHttpDispatcher& FHttpLogModel::FindDispatcher(uint64 Handle)
{
	for (const FHttpDispatcher& Dispatcher : Dispatchers)
	{
		if (Dispatcher.Handle == Handle)
		{
			return Dispatcher;
		}
	}

	return Dispatchers[0]; // Unknown
}

const FHttpLogModel::FCategory& FHttpLogModel::FindCategory(uint8 CategoryValue) const
{
	for (const FCategory& Category : Categories)
	{
		if (Category.Value == CategoryValue)
		{
			return Category;
		}
	}

	return Categories[0]; // Unknown
}

const TCHAR* FHttpLogModel::GetPackageName(const FIoChunkId& ChunkId) const
{
	return PackageNameCache.Get(ChunkId);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<IHttpLogModel> MakeHttpLogModel(IAnalysisSession& Session)
{
	return MakeShared<FHttpLogModel>(Session);
}

} // namespace UE:HttpInsights
