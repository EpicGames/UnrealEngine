// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/AnsiString.h"
#include "IO/IoChunkId.h"
#include "IO/PackageId.h"
#include "Math/NumericLimits.h"
#include "Templates/Function.h"
#include "TraceServices/Model/AnalysisSession.h"

namespace UE::HttpInsights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
struct FHttpDispatcher
{
	const uint64	Handle = 0;
	const FString	Name;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
struct FHttpChunkRange
{
	FHttpChunkRange*	Next		= nullptr;
	FIoChunkId			ChunkId		= FIoChunkId::InvalidChunkId;
	uint32				Start		= MAX_uint32;
	uint32				End			= MIN_uint32;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
struct FHttpRequest
{
	const FHttpDispatcher&	Dispatcher;
	double					StartTime		= 0.0;
	double					CompletionTime	= 0.0;
	uint32					StatusCode		= 0;
	uint32					ContentLength	= 0;
	uint32					SeqNo			= 0;
	FHttpChunkRange*		ChunkRanges		= nullptr;
	FString					Host;
	FString					Url;
	const FString*			CategoryName	= nullptr;
	uint8					Category		= MAX_uint8;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
struct FHttpDispatcherCreated
{
	uint64		Dispatcher = 0;
	FAnsiString	Name;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
struct FHttpCategoryCreated
{
	uint8		Category = MAX_uint8;
	FAnsiString	Name;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
struct FHttpRequestStarted
{
	uint64					Dispatcher		= 0;
	uint64					Request			= 0;
	double					StartTime		= 0.0;
	int32					Priority		= 0;
	uint32					Category		= 0;
	FHttpChunkRange*		ChunkRanges		= nullptr;
	FAnsiString				Url;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
struct FHttpChunkRangeAdded
{
	uint64		Request		= 0;
	uint32		Start		= MAX_uint32;
	uint32		End			= MIN_uint32;
	FIoChunkId	ChunkId		= FIoChunkId::InvalidChunkId;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
struct FHttpRequestCompleted
{
	uint64		Request			= 0;
	double		CompletionTime	= 0.0;
	uint32		StatusCode		= 0;
	uint32		ContentLength	= 0;
	FAnsiString	Host;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
struct FPackageIdMapped
{
	FPackageId		PackageId;
	const TCHAR*	PackageName = nullptr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
class IHttpLogModel
	: public TraceServices::IProvider
{
public:
	inline static const FName ProviderName = TEXT("HttpLogProvider");
	using FReadLogFunc = TFunctionRef<void(const FHttpRequest&)>;

	virtual					~IHttpLogModel() = default;
	virtual void			ProcesEvent(FHttpDispatcherCreated&&) = 0;
	virtual void			ProcesEvent(FHttpCategoryCreated&&) = 0;
	virtual void 			ProcesEvent(FHttpRequestStarted&&) = 0;
	virtual void 			ProcesEvent(FHttpChunkRangeAdded&&) = 0;
	virtual void 			ProcesEvent(FHttpRequestCompleted&&) = 0;
	virtual void 			ProcesEvent(FPackageIdMapped&&) = 0;
	virtual int32			IterateLog(int32 StartIndex, FReadLogFunc&&) const = 0;
	virtual const TCHAR*	GetPackageName(const FIoChunkId&) const = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
TSharedPtr<IHttpLogModel> MakeHttpLogModel(TraceServices::IAnalysisSession& Session);

} // namespace UE:HttpInsights
