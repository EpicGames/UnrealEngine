// Copyright Epic Games, Inc. All Rights Reserved.

#include "HttpInsights/ModuleInterface.h"
#include "Model.h"

#include "Features/IModularFeatures.h"
#include "Insights/IUnrealInsightsModule.h"
#include "Insights/ITimingViewExtender.h"
#include "Modules/ModuleManager.h"
#include "Trace/Analyzer.h"
#include "TraceServices/ModuleService.h"

#define LOCTEXT_NAMESPACE "HttpInsightsModule"

DEFINE_LOG_CATEGORY(LogHttpInsights);

namespace UE::HttpInsights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
namespace View
{

////////////////////////////////////////////////////////////////////////////////////////////////////
void RegisterTimingProfilerExtensions(
	FInsightsMajorTabExtender& InOutExtender,
	Insights::Timing::ITimingViewExtender* TimingViewExtender);

////////////////////////////////////////////////////////////////////////////////////////////////////
TUniquePtr<Insights::Timing::ITimingViewExtender> MakeTimingViewExtender();

}

////////////////////////////////////////////////////////////////////////////////////////////////////
TUniquePtr<UE::Trace::IAnalyzer> MakeHttpAnalyzer(
	TraceServices::IAnalysisSession& Session,
	class IHttpLogModel& LogModel);

////////////////////////////////////////////////////////////////////////////////////////////////////
class FTraceModule
	: public TraceServices::IModule
{
	static const FName		ModuleName;

public:
	virtual					~FTraceModule() = default;
	virtual void			GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo) override;
	virtual void			GetLoggers(TArray<const TCHAR *>& OutLoggers) override;
	virtual bool			ShouldBeEnabledByDefault() const override { return true; }
	virtual const TCHAR*	GetCommandLineArgument() override { return TEXT("http"); }
	virtual void			OnAnalysisBegin(TraceServices::IAnalysisSession& Session) override;
	virtual void			GenerateReports(
								const TraceServices::IAnalysisSession& Session,
								const TCHAR* CmdLine,
								const TCHAR* OutputDirectory) override;
};

const FName FTraceModule::ModuleName = FName("TraceModule_Http");

////////////////////////////////////////////////////////////////////////////////////////////////////
void FTraceModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
{
	OutModuleInfo.Name = ModuleName;
	OutModuleInfo.DisplayName = TEXT("Http");
}

void FTraceModule::GetLoggers(TArray<const TCHAR *>& OutLoggers)
{
	OutLoggers.Add(TEXT("Http"));
}

void FTraceModule::OnAnalysisBegin(TraceServices::IAnalysisSession& Session)
{
	TSharedPtr<IHttpLogModel> LogModel = MakeHttpLogModel(Session);
	TUniquePtr<UE::Trace::IAnalyzer> Analyzer = MakeHttpAnalyzer(Session, *LogModel);

	Session.AddProvider(IHttpLogModel::ProviderName, LogModel);
	Session.AddAnalyzer(Analyzer.Release());
}

void FTraceModule::GenerateReports(
	const TraceServices::IAnalysisSession& Session,
	const TCHAR* CmdLine,
	const TCHAR* OutputDirectory)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////
class FHttpInsightsModule
	: public IHttpInsightsModule
{
public:
	virtual void	StartupModule() override;
	virtual void	ShutdownModule() override;

private:
	void			RegisterTimingProfilerExtensions(FInsightsMajorTabExtender& InOutExtender);

	TUniquePtr<FTraceModule>							TraceModule;
	TUniquePtr<Insights::Timing::ITimingViewExtender>	TimingViewExtender;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
void FHttpInsightsModule::StartupModule()
{
	TraceModule = MakeUnique<FTraceModule>();
	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, TraceModule.Get());
	TimingViewExtender = View::MakeTimingViewExtender();
	IModularFeatures::Get().RegisterModularFeature(Insights::Timing::TimingViewExtenderFeatureName, TimingViewExtender.Get());

	IUnrealInsightsModule& Insights = FModuleManager::LoadModuleChecked<IUnrealInsightsModule>("TraceInsights");
	Insights.OnRegisterMajorTabExtension(FInsightsManagerTabs::TimingProfilerTabId)
		.AddRaw(this, &FHttpInsightsModule::RegisterTimingProfilerExtensions);
}

void FHttpInsightsModule::ShutdownModule()
{
	if (IUnrealInsightsModule* Insights = FModuleManager::GetModulePtr<IUnrealInsightsModule>("TraceInsights"))
	{
		Insights->OnRegisterMajorTabExtension(FInsightsManagerTabs::TimingProfilerTabId)
			.RemoveAll(this);
	}

	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, TraceModule.Get());
	IModularFeatures::Get().UnregisterModularFeature(Insights::Timing::TimingViewExtenderFeatureName, TimingViewExtender.Get());
}

void FHttpInsightsModule::RegisterTimingProfilerExtensions(FInsightsMajorTabExtender& InOutExtender)
{
	View::RegisterTimingProfilerExtensions(InOutExtender, TimingViewExtender.Get());
}

} // namespace UE:HttpInsights

IMPLEMENT_MODULE(UE::HttpInsights::FHttpInsightsModule, HttpInsights)

#undef LOCTEXT_NAMESPACE
