// Copyright Epic Games, Inc. All Rights Reserved.

#include "Model.h"

#include "Delegates/Delegate.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Insights/IUnrealInsightsModule.h"
#include "Insights/ITimingViewExtender.h"
#include "Insights/ITimingViewSession.h"
#include "Math/Color.h"
#include "Math/Interval.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/EnumClassFlags.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "HttpInsightsView"

namespace UE::HttpInsights::View
{

using namespace Insights::Timing;
using namespace TraceServices;

////////////////////////////////////////////////////////////////////////////////////////////////////
class FHttpTimingViewSession
{
public:
							FHttpTimingViewSession() = default;
	bool					IsValid() const { return TimingSession != nullptr && AnalysisSession != nullptr; }
	ITimingViewSession&		GetTimingViewSession() const { return *TimingSession; }
	const IAnalysisSession&	GetAnalysisSession() const { return *AnalysisSession; }

private:
	friend class FHttpTimingViewExtender;

	ITimingViewSession*		TimingSession = nullptr;
	const IAnalysisSession*	AnalysisSession = nullptr;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
class IHttpTimingView
{
public:
	virtual void OnBeginSession(FHttpTimingViewSession& Session) = 0;
	virtual void OnEndSession(FHttpTimingViewSession& Session) = 0;
	virtual void OnTickSession(FHttpTimingViewSession&) = 0;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
class FHttpTimingViewExtender
	: public Insights::Timing::ITimingViewExtender
{
public:
	virtual void	OnBeginSession(ITimingViewSession& TimingViewSession) override;
	virtual void	OnEndSession(ITimingViewSession& TimingViewSession) override;
	virtual void	Tick(ITimingViewSession& TimingViewSession, const IAnalysisSession& AnalysisSession) override;
	void			AddView(IHttpTimingView* View);
	void			RemoveView(IHttpTimingView* View);

private:
	FHttpTimingViewSession		CurrentSession;
	TArray<IHttpTimingView*>	Views;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
void FHttpTimingViewExtender::OnBeginSession(ITimingViewSession& TimingViewSession)
{
	if (TimingViewSession.GetName() != FInsightsManagerTabs::TimingProfilerTabId)
	{
		return;
	}

	check(CurrentSession.TimingSession == nullptr);
	check(CurrentSession.AnalysisSession == nullptr);
	CurrentSession.TimingSession = &TimingViewSession;
}

void FHttpTimingViewExtender::OnEndSession(ITimingViewSession& TimingViewSession)
{
	if (TimingViewSession.GetName() != FInsightsManagerTabs::TimingProfilerTabId)
	{
		return;
	}

	if (CurrentSession.TimingSession != &TimingViewSession) 
	{
		return;
	}

	if (CurrentSession.IsValid())
	{
		for (IHttpTimingView* View : Views)
		{
			View->OnEndSession(CurrentSession);
		}
	}
	
	CurrentSession = FHttpTimingViewSession();
}

void FHttpTimingViewExtender::Tick(ITimingViewSession& TimingViewSession, const IAnalysisSession& AnalysisSession)
{
	if (TimingViewSession.GetName() != FInsightsManagerTabs::TimingProfilerTabId)
	{
		return;
	}

	if (CurrentSession.TimingSession != &TimingViewSession)
	{
		return;
	}

	if (CurrentSession.AnalysisSession == nullptr)
	{
		CurrentSession.AnalysisSession = &AnalysisSession;
		for (IHttpTimingView* View : Views)
		{
			View->OnBeginSession(CurrentSession);
		}
	}

	for (IHttpTimingView* View : Views)
	{
		View->OnTickSession(CurrentSession);
	}
}

void FHttpTimingViewExtender::AddView(IHttpTimingView* View)
{
	Views.Add(View);
	if (CurrentSession.IsValid())
	{
		View->OnBeginSession(CurrentSession);
	}
}

void FHttpTimingViewExtender::RemoveView(IHttpTimingView* View)
{
	Views.Remove(View);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
struct FHttpRequestViewModelData
{
	FHttpRequestViewModelData(const IHttpLogModel& InModel, const FHttpRequest& InRequest)
		: Model(InModel)
		, Request(InRequest)
		, FullUrl(InRequest.Host / InRequest.Url)
	{
		StatusColor = Request.StatusCode > 199 && Request.StatusCode < 300
			? FLinearColor::Green
			: FLinearColor::Red;

		// Just some simple colors for IAS/IAD
		CategoryColor = InRequest.Category == 0
			? FLinearColor(1.0f, 0.0f, 1.0f)
			: InRequest.Category == 1
				? FLinearColor(0.0f, 1.0f, 1.0f)
				: FLinearColor::Gray;
	}

	bool	HasChunkRanges() const { return Request.ChunkRanges != nullptr; }
	bool	MatchesText(const FString& SearchText);
	FText	GetChunkRangeText(const FHttpChunkRange& ChunkRange) const;

	const IHttpLogModel&	Model;
	const FHttpRequest&		Request;
	const FString			FullUrl;
	FLinearColor			StatusColor;
	FLinearColor			CategoryColor;
};
using FHttpRequestViewModel = TSharedPtr<FHttpRequestViewModelData>;

////////////////////////////////////////////////////////////////////////////////////////////////////
bool FHttpRequestViewModelData::MatchesText(const FString& SearchText)
{
	for (FHttpChunkRange* It = Request.ChunkRanges; It != nullptr; It = It->Next)
	{
		if (const TCHAR* PackageName = Model.GetPackageName(It->ChunkId))
		{
			//TODO: Fix
			FString Tmp(PackageName);
			if (Tmp.Contains(SearchText))
			{
				return true;
			}
		}
	}

	return false;
}

FText FHttpRequestViewModelData::GetChunkRangeText(const FHttpChunkRange& ChunkRange) const
{
	if (const TCHAR* PackageName = Model.GetPackageName(ChunkRange.ChunkId))
	{
		return FText::FromString(
			FString::Printf(TEXT("%s [%s] [%u - %u] %s"),
				PackageName,
				*LexToString(ChunkRange.ChunkId.GetChunkType()),
				ChunkRange.Start, ChunkRange.End,
				*FText::AsMemory(ChunkRange.End - ChunkRange.Start).ToString()));
	}
	else
	{
		return FText::FromString(
			FString::Printf(TEXT("%s [%s] [%u - %u] %s"),
				*LexToString(ChunkRange.ChunkId),
				*LexToString(ChunkRange.ChunkId.GetChunkType()),
				ChunkRange.Start, ChunkRange.End,
				*FText::AsMemory(ChunkRange.End - ChunkRange.Start).ToString()));
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
class SHttpRequestTableRow
	: public SMultiColumnTableRow<FHttpRequestViewModel>
{
public:
	using Super = SMultiColumnTableRow<FHttpRequestViewModel>;

	inline static const FName TimeColumn			= FName("Time");
	inline static const FName StatusColumn			= FName("Status");
	inline static const FName DurationColumn		= FName("Duration");
	inline static const FName ContentLengthColumn	= FName("ContentLength");
	inline static const FName UrlColumn				= FName("Url");
	inline static const FName CategoryColumn		= FName("Category");

	SLATE_BEGIN_ARGS(SHttpRequestTableRow) { }
	SLATE_END_ARGS()

	void Construct(
		const FArguments& Args,
		const TSharedRef<STableViewBase>& OwnerTable,
		FHttpRequestViewModel InViewModel)
	{
		ViewModel = InViewModel;
		Super::Construct(Super::FArguments().Padding(1.0f), OwnerTable);
	}

	static TSharedPtr<SHeaderRow>	CreateHeaderRow();
private:
	TSharedRef<SWidget>				GenerateWidgetForColumn(const FName& Column);

	FHttpRequestViewModel ViewModel;
};

TSharedPtr<SHeaderRow> SHttpRequestTableRow::CreateHeaderRow()
{
	return SNew(SHeaderRow)
		+SHeaderRow::Column(TimeColumn)
		.DefaultLabel(FText::FromString("Time"))
		.HAlignHeader(EHorizontalAlignment::HAlign_Center)
		.FixedWidth(64)
		+SHeaderRow::Column(StatusColumn)
		.DefaultLabel(LOCTEXT("StatusColumnName", "Status"))
		.HAlignHeader(EHorizontalAlignment::HAlign_Center)
		.FixedWidth(64)
		+SHeaderRow::Column(DurationColumn)
		.DefaultLabel(LOCTEXT("DurationColumnName", "Duration"))
		.HAlignHeader(EHorizontalAlignment::HAlign_Center)
		.FixedWidth(72)
		+SHeaderRow::Column(ContentLengthColumn)
		.DefaultLabel(LOCTEXT("ContentLengthColumnName", "Content-Length"))
		.HAlignHeader(EHorizontalAlignment::HAlign_Left)
		.FixedWidth(128)
		+SHeaderRow::Column(UrlColumn)
		.DefaultLabel(LOCTEXT("UrlColumnName", "Url"))
		.HAlignHeader(EHorizontalAlignment::HAlign_Left)
		+SHeaderRow::Column(CategoryColumn)
		.DefaultLabel(LOCTEXT("CategoryColumnName", "Category"))
		.HAlignHeader(EHorizontalAlignment::HAlign_Center);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
TSharedRef<SWidget> SHttpRequestTableRow::GenerateWidgetForColumn(const FName& Column)
{
	const float PaddingLeft = 12.0f;

	if (Column == TimeColumn)
	{
		return SNew(SBox)
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::AsTimespan(FTimespan::FromSeconds(ViewModel->Request.CompletionTime)))
			];
	}
	else if (Column == StatusColumn)
	{
		return SNew(SBox)
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::AsNumber(ViewModel->Request.StatusCode))
				.ColorAndOpacity(ViewModel->StatusColor)
			];
	}
	else if (Column == DurationColumn)
	{
		return SNew(SBox)
			.HAlign(HAlign_Left)
			.Padding(PaddingLeft, 0, 0, 0)
			[
				SNew(STextBlock)
				.Text(FText::AsNumber(ViewModel->Request.CompletionTime - ViewModel->Request.StartTime))
			];
	}
	else if (Column == ContentLengthColumn)
	{
		return SNew(SBox)
			.HAlign(HAlign_Left)
			.Padding(PaddingLeft, 0, 0, 0)
			[
				SNew(STextBlock)
					.Text(FText::AsMemory(ViewModel->Request.ContentLength))
			];
	}
	else if (Column == UrlColumn)
	{
		if (ViewModel->HasChunkRanges())
		{
			TSharedRef<SVerticalBox> Vertical = SNew(SVerticalBox)
				+SVerticalBox::Slot()
				[
					SNew(STextBlock)
					.Text(FText::FromString(ViewModel->FullUrl))
				];

			for (FHttpChunkRange* It = ViewModel->Request.ChunkRanges; It != nullptr; It = It->Next)
			{
				Vertical->AddSlot()
				[
					SNew(STextBlock)
					.Text(ViewModel->GetChunkRangeText(*It))
				];
			}

			return SNew(SBox).Padding(PaddingLeft, 0, 0, 0)[Vertical];
		}

		return SNew(SBox)
			.Padding(PaddingLeft, 0, 0, 0)
			[
				SNew(STextBlock)
				.Text(FText::FromString(ViewModel->FullUrl))
			];
	}
	else if (Column == CategoryColumn)
	{
		return SNew(SBox)
			.Padding(PaddingLeft, 0, 0, 0)
			[
				SNew(STextBlock)
				.Text(FText::FromString(*ViewModel->Request.CategoryName))
				.ColorAndOpacity(ViewModel->CategoryColor)
			];
	}

	return SNullWidget::NullWidget;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
class SHttpRequestTableView
	: public SListView<FHttpRequestViewModel>
{
public:
	using Super				= SListView<FHttpRequestViewModel>;

							SLATE_BEGIN_ARGS(SHttpRequestTableView) { }
								SLATE_ARGUMENT(TArray<FHttpRequestViewModel>*, ListItemsSource)
							SLATE_END_ARGS()
	
	void					Construct(const FArguments& Args);
	void					Refresh();

private:
	virtual FReply			OnMouseWheel(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply			OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
	void					HandleUserScrolled(double Offset);
	TSharedRef<ITableRow>	OnGenerateRow(FHttpRequestViewModel ViewModel, const TSharedRef<STableViewBase>& OwnerTable)
							{
								return SNew(SHttpRequestTableRow, OwnerTable, ViewModel);
							}

	bool bAutoScroll = true;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
void SHttpRequestTableView::Construct(const FArguments& InArgs)
{
	Super::Construct
	(
		Super::FArguments()
		.ScrollbarVisibility(EVisibility::Visible)
		.ListItemsSource(InArgs._ListItemsSource)
		.OnGenerateRow(this, &SHttpRequestTableView::OnGenerateRow)
		.HeaderRow(SHttpRequestTableRow::CreateHeaderRow())
		.OnListViewScrolled(this, &SHttpRequestTableView::HandleUserScrolled)
	);
}

void SHttpRequestTableView::Refresh()
{
	Super::RequestListRefresh();
	if (bAutoScroll)
	{
		Super::ScrollToBottom();
	}
}

FReply SHttpRequestTableView::OnMouseWheel(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	bAutoScroll = false;
	return Super::OnMouseWheel(InGeometry, InMouseEvent);
}

FReply SHttpRequestTableView::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	if (bAutoScroll == false)
	{
		bAutoScroll = InKeyEvent.GetKey() == EKeys::PageDown;
	}
	return Super::OnKeyDown(InGeometry, InKeyEvent);
}

void SHttpRequestTableView::HandleUserScrolled(double Offset)
{
	if (bAutoScroll && IsUserScrolling())
	{
		bAutoScroll = false;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
enum class EHttpCategoryFilter : uint8
{
	None		= 0,
	Streaming	= (1 << 0),
	Install		= (1 << 1),
	All			= Streaming | Install
};
ENUM_CLASS_FLAGS(EHttpCategoryFilter)

////////////////////////////////////////////////////////////////////////////////////////////////////
class SHttpFilterBar
	: public SCompoundWidget
{
public:
	DECLARE_MULTICAST_DELEGATE(FFilterChangedDelegate);

								SLATE_BEGIN_ARGS(SHttpFilterBar) { }
								SLATE_END_ARGS()
	void						Construct(const FArguments& Args);
	void						SetSelectedTimeRange(FDoubleInterval TimeRange);
	bool						ShouldFilter(const FHttpRequestViewModel& Vm) const;
	FFilterChangedDelegate&		OnFilterChanged() { return FilterChanged; }

private:
	void						UpdateCategoryFilter(EHttpCategoryFilter Filter);
	bool						IsCategoryFilterEnabled(EHttpCategoryFilter Filer);
	void						HandleSearchTextChanged(const FText& NewSearchText);
	TSharedRef<SWidget>			MakeCategoryMenu();

	TSharedPtr<SEditableTextBox>	SearchTextBox;
	FFilterChangedDelegate			FilterChanged;
	EHttpCategoryFilter				CurrentCategoryFilter = EHttpCategoryFilter::All;
	FDoubleInterval					CurrentTimeRange;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
void SHttpFilterBar::Construct(const FArguments& Args)
{
	FSlimHorizontalToolBarBuilder ToolbarBuilder(TSharedPtr<const FUICommandList>(), FMultiBoxCustomization::None);
	ToolbarBuilder.BeginSection("Filters");
	{
		ToolbarBuilder.AddComboButton(
			FUIAction(),
			FOnGetContent::CreateSP(this, &SHttpFilterBar::MakeCategoryMenu),
			LOCTEXT("HttpCategoryText", "Category Filter"),
			LOCTEXT("HttpThresholdToolTip", "Filter HTTP request by category."),
			FSlateIcon(),
			false
		);
	}
	ToolbarBuilder.EndSection();

	ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			ToolbarBuilder.MakeWidget()
		]
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			SNew(SBox)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.Padding(FMargin(11.0f, 0.0f))
			[
				SAssignNew(SearchTextBox, SEditableTextBox)
				.HintText(NSLOCTEXT("HttpRequestSearchBox", "Search", "Search..."))
				.SelectAllTextWhenFocused(true)
				.OnTextChanged(this, &SHttpFilterBar::HandleSearchTextChanged)
			]
		]
	];
}

void SHttpFilterBar::SetSelectedTimeRange(FDoubleInterval TimeRange)
{
	CurrentTimeRange = TimeRange;
	FilterChanged.Broadcast();
}

bool SHttpFilterBar::ShouldFilter(const FHttpRequestViewModel& Vm) const
{
	//TODO: Category filtering is currently hardcoded see OnDemandHttpIoDispatcher.h
	if (!EnumHasAnyFlags(CurrentCategoryFilter, EHttpCategoryFilter::Streaming) && Vm->Request.Category == 0)
	{
		return true;
	}

	if (!EnumHasAnyFlags(CurrentCategoryFilter, EHttpCategoryFilter::Install) && Vm->Request.Category == 1)
	{
		return true;
	}

	if (CurrentTimeRange.IsValid())
	{
		if (CurrentTimeRange.Contains(Vm->Request.CompletionTime) == false)
		{
			return true;
		}
	}

	const FText SearchText = SearchTextBox->GetText();
	if (SearchText.IsEmpty() == false)
	{
		const FString Text = SearchText.ToString();
		if (Vm->MatchesText(Text) == false)
		{
			return true;
		}
	}

	return false;
}

TSharedRef<SWidget>	SHttpFilterBar::MakeCategoryMenu()
{
	struct FCategoryFilter
	{
		EHttpCategoryFilter	Value;
		FText				Label;
		FText				ToolTip;
	};

	const FCategoryFilter Filters[]
	{
		{
			EHttpCategoryFilter::Streaming,
			LOCTEXT("HttpCategoryMenuStreaming", "Streaming"),
			LOCTEXT("HttpCategoryStreaming_Tooltip", "Filter HTTP requests by Streaming category.")
		},
		{
			EHttpCategoryFilter::Install,
			LOCTEXT("HttpCategoryMenuInstall", "Install"),
			LOCTEXT("HttpCategoryInstall_Tooltip", "Filter HTTP requests by Install category.")
		}
	};

	FMenuBuilder MenuBuilder(true, nullptr, nullptr, false, &FCoreStyle::Get(), false);
	MenuBuilder.BeginSection("HttpCategory");
	for (int32 Idx = 0, Count = UE_ARRAY_COUNT(Filters); Idx < Count; ++Idx)
	{
		const FCategoryFilter& Filter = Filters[Idx];
		const TSharedRef<SWidget> TextBlock = 
			SNew(STextBlock)
			.Text(Filter.Label);

		MenuBuilder.AddMenuEntry(
			FUIAction(
				FExecuteAction::CreateSP(this, &SHttpFilterBar::UpdateCategoryFilter, Filter.Value),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SHttpFilterBar::IsCategoryFilterEnabled, Filter.Value)),
			TextBlock,
			NAME_None,
			Filter.ToolTip,
			EUserInterfaceActionType::Check
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SHttpFilterBar::UpdateCategoryFilter(EHttpCategoryFilter Filter)
{
	if (EnumHasAnyFlags(CurrentCategoryFilter, Filter))
	{
		EnumRemoveFlags(CurrentCategoryFilter, Filter);
	}
	else
	{
		EnumAddFlags(CurrentCategoryFilter, Filter);
	}

	FilterChanged.Broadcast();
}

bool SHttpFilterBar::IsCategoryFilterEnabled(EHttpCategoryFilter Filter)
{
	return EnumHasAnyFlags(CurrentCategoryFilter, Filter);
}

void SHttpFilterBar::HandleSearchTextChanged(const FText& NewSearchText)
{
	FilterChanged.Broadcast();
}

////////////////////////////////////////////////////////////////////////////////////////////////////
class SHttpInsightsTab
	: public SCompoundWidget
	, public IHttpTimingView
{
public:
	static const FName		TabId;

							SLATE_BEGIN_ARGS(SHttpInsightsTab) { }
								SLATE_ARGUMENT(FHttpTimingViewExtender*, TimingViewExtender)
							SLATE_END_ARGS()
							~SHttpInsightsTab();
	void					Construct(const FArguments& InArgs);

	virtual void			OnBeginSession(FHttpTimingViewSession& Session);
	virtual void			OnEndSession(FHttpTimingViewSession& Session);
	virtual void			OnTickSession(FHttpTimingViewSession& Session) { }
	virtual void			Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	void					OnTimeMarkerChanged(ETimeChangedFlags Flags, double TimeMarker);
	void					OnTimeSelectionChanged(ETimeChangedFlags Flags, double StartTime, double EndTime);

	FHttpTimingViewExtender*			TimingViewExtender = nullptr;
	FHttpTimingViewSession*				CurrentSession = nullptr;
	TArray<FHttpRequestViewModel>		HttpRequests;
	TArray<FHttpRequestViewModel>		AllHttpRequests;
	TSharedPtr<SHttpFilterBar>			FilterBar;
	TSharedPtr<SHttpRequestTableView>	RequestTable;
	bool								bFilterChanged = false;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
const FName SHttpInsightsTab::TabId = FName("HttpInsightsTab");

SHttpInsightsTab::~SHttpInsightsTab()
{
	if (CurrentSession != nullptr)
	{
		CurrentSession->GetTimingViewSession()
			.OnTimeMarkerChanged().RemoveAll(this);
		CurrentSession->GetTimingViewSession()
			.OnSelectionChanged().RemoveAll(this);
	}

	if (TimingViewExtender != nullptr)
	{
		TimingViewExtender->RemoveView(this);
	}
}

void SHttpInsightsTab::Construct(const FArguments& InArgs)
{
	TimingViewExtender = InArgs._TimingViewExtender;
	check(TimingViewExtender != nullptr);

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(FilterBar, SHttpFilterBar)
		]
		+SVerticalBox::Slot()
		[
			SAssignNew(RequestTable, SHttpRequestTableView)
			.ListItemsSource(&HttpRequests)
		]
	];

	TimingViewExtender->AddView(this);
	FilterBar->OnFilterChanged().AddLambda([this]
	{
		bFilterChanged = true;
	});
}

void SHttpInsightsTab::OnBeginSession(FHttpTimingViewSession& Session)
{
	check(CurrentSession == nullptr);
	CurrentSession = &Session;
	CurrentSession->GetTimingViewSession()
		.OnTimeMarkerChanged().AddRaw(this, &SHttpInsightsTab::OnTimeMarkerChanged);
	CurrentSession->GetTimingViewSession()
		.OnSelectionChanged().AddRaw(this, &SHttpInsightsTab::OnTimeSelectionChanged);
}

void SHttpInsightsTab::OnEndSession(FHttpTimingViewSession& Session)
{
	if (CurrentSession == &Session)
	{
		CurrentSession->GetTimingViewSession()
			.OnTimeMarkerChanged().RemoveAll(this);
		CurrentSession->GetTimingViewSession()
			.OnSelectionChanged().RemoveAll(this);
		CurrentSession = nullptr;
		HttpRequests.Empty();
		AllHttpRequests.Empty();
	}
}

void SHttpInsightsTab::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (CurrentSession == nullptr)
	{
		HttpRequests.Empty();
		return;
	}

	const IHttpLogModel* HttpLog = CurrentSession->GetAnalysisSession().ReadProvider<IHttpLogModel>(IHttpLogModel::ProviderName);
	if (HttpLog == nullptr)
	{
		HttpRequests.Empty();
		return;
	}

	const int32 Num		= AllHttpRequests.Num();
	const int32 NewNum	= HttpLog->IterateLog(Num, [this, HttpLog](const FHttpRequest& Request)
	{
		FHttpRequestViewModel Vm = MakeShared<FHttpRequestViewModelData>(*HttpLog, Request);
		AllHttpRequests.Add(Vm);

		if (!bFilterChanged && FilterBar->ShouldFilter(Vm) == false)
		{
			HttpRequests.Add(Vm);
		}
	});

	const bool bRefresh = bFilterChanged || (Num != NewNum);

	if (bFilterChanged)
	{
		bFilterChanged = false;
		HttpRequests.Empty(AllHttpRequests.Num());
		for (const FHttpRequestViewModel& Request : AllHttpRequests)
		{
			if (FilterBar->ShouldFilter(Request) == false)
			{
				HttpRequests.Add(Request);
			}
		}
	}

	if (bRefresh)
	{
		RequestTable->Refresh();
	}
}

void SHttpInsightsTab::OnTimeMarkerChanged(ETimeChangedFlags Flags, double TimeMarker)
{
	//TODO: Find closest entry to time marker
}

void SHttpInsightsTab::OnTimeSelectionChanged(ETimeChangedFlags Flags, double StartTime, double EndTime)
{
	if (Flags == UE::Insights::Timing::ETimeChangedFlags::None)
	{
		const bool bValidRange = (EndTime > StartTime) && !FMath::IsNearlyEqual(StartTime, EndTime);
		FilterBar->SetSelectedTimeRange(bValidRange ? FDoubleInterval(StartTime, EndTime) : FDoubleInterval());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
void RegisterTimingProfilerExtensions(
	FInsightsMajorTabExtender& InOutExtender,
	Insights::Timing::ITimingViewExtender* TimingViewExtender)
{
	FMinorTabConfig& MinorTabConfig = InOutExtender.AddMinorTabConfig();

	MinorTabConfig.TabId			= SHttpInsightsTab::TabId;
	MinorTabConfig.TabLabel			= LOCTEXT("HttpInsightsTabTitle", "HTTP Insights");
	MinorTabConfig.TabTooltip		= LOCTEXT("HttpInsightsTabTitleTooltip", "View and diagnose HTTP traffic.");
	MinorTabConfig.TabIcon			= FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Plugin.TreeItem");
	MinorTabConfig.WorkspaceGroup	= InOutExtender.GetWorkspaceGroup();
	MinorTabConfig.OnSpawnTab		= FOnSpawnTab::CreateLambda([TimingViewExtender](const FSpawnTabArgs& Args)
	{
		return SNew(SDockTab)
			.ShouldAutosize(false)
			.TabRole(ETabRole::PanelTab)
			[
				SNew(SHttpInsightsTab)
				.TimingViewExtender(reinterpret_cast<FHttpTimingViewExtender*>(TimingViewExtender))
			];
	});
}

////////////////////////////////////////////////////////////////////////////////////////////////////
TUniquePtr<Insights::Timing::ITimingViewExtender> MakeTimingViewExtender()
{
	return MakeUnique<FHttpTimingViewExtender>();
}

} // namespace UE::HttpInsights::View

#undef LOCTEXT_NAMESPACE
