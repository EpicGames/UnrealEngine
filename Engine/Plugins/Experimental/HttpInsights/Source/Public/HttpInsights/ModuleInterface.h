// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Modules/ModuleInterface.h"
#include "Logging/LogMacros.h"
#include "Logging/LogVerbosity.h"

#define UE_API HTTPINSIGHTS_API

UE_API DECLARE_LOG_CATEGORY_EXTERN(LogHttpInsights, Log, All);

namespace UE::HttpInsights
{

////////////////////////////////////////////////////////////////////////////////////////////////////
class IHttpInsightsModule
	: public IModuleInterface
{
public:
};

} // namespace UE:HttpInsights

#undef UE_API
